// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;

namespace UnrealHitchParser
{
    class Program
    {
        static float HitchThreshold = 3.0f;
        static void Main(string[] args)
        {
            if (args.Length < 1)
            {
                return;
            }
            if (args.Length >= 2)
            {
                HitchThreshold = float.Parse(args[1]);
            }
            if (File.Exists(args[0]))
            {
                GetHitchesFromLog(args[0]);
            }
        }

        static List<string> GetHitchesFromLog(string FileLoc)
        {

            bool bRecordingStack = false;

            // The stuff we actually want to write to file.
            List<string> RelevantData = new List<string>();
            // For ease of tracking lines where hitches start in log.
            int LineNum = 0;
            // For parsing our dumphitch whitespace delimitation.
            int LastNumPrecedingWhitespaces = 0;
            // Our BuildAStack!
            Stack<string> NamespacedHitch = new Stack<string>();

            string CachedDurationString = "";
            string CallStack = "";

            StreamReader NewReader = new StreamReader(FileLoc);

            while (!NewReader.EndOfStream)
            {
                string TempString = NewReader.ReadLine();
                LineNum++;
                if (bRecordingStack)
                {
                    string[] stringSeparators = new string[] { "LogStats:" };
                    string[] SplitString = TempString.Split(stringSeparators, StringSplitOptions.RemoveEmptyEntries);
                    string StringToUse = "";
                    if (SplitString.Length > 0)
                    {
                        StringToUse = SplitString.Last();
                    }
                    StringToUse += "\n";

                    int NumSpaces = StringToUse.TakeWhile(Char.IsWhiteSpace).Count();
                    // Whitespace indentation starts with 3 spaces. This is gross in general.
                    if (NumSpaces > 2)
                    {
                        // If the number of spaces is lower than on the previous line, we recently hit a leaf and need to back out all
                        // irrelevant namespacing, while also adding the hitch to the report if it meets our threshold.
                        if (NumSpaces <= LastNumPrecedingWhitespaces)
                        {
                            // Have to append in reverse because it's a stack.
                            string NamespacedThing = string.Join(".", NamespacedHitch.Reverse());
                            NamespacedThing += " " + CachedDurationString + "ms\n";
                            if (float.Parse(CachedDurationString) > HitchThreshold)
                            {
                                RelevantData.Add(NamespacedThing);
                            }
                            // Once we've added the leaf, pop stuff off the stack until we are bacl tp the [rp[er de[th/
                            while (NumSpaces <= LastNumPrecedingWhitespaces)
                            {
                                LastNumPrecedingWhitespaces -= 2;
                                if (NamespacedHitch.Count > 0)
                                {
                                    NamespacedHitch.Pop();
                                }
                            }
                        }
                        LastNumPrecedingWhitespaces = NumSpaces;
                        // StringToUse is in a Duration - Category format.
                        string CategoryString = StringToUse.Split('-')[1].Trim();
                        // We only want the last relevant thing in each timer, so cull unnecessary pathing and namespacing.
                        if (CategoryString.Contains('/'))
                        {
                            CategoryString = CategoryString.Split('/').Last();
                        }
                        if (CategoryString.Contains('.'))
                        {
                            CategoryString = CategoryString.Split('.').Last();
                        }
                        NamespacedHitch.Push(CategoryString);
                        string[] DurationSeparator = new string[] {"ms" };
                        // Save our duration for next iteration because we can't tell if this is a leaf yet.
                        CachedDurationString = StringToUse.Split('-')[0].Split(DurationSeparator, StringSplitOptions.RemoveEmptyEntries)[0].Trim();
   
                    } 
                    CallStack += StringToUse;
                    // This many dashes means the end of a callstack! Blech.
                    if (TempString.Contains("-----------------------------------"))
                    {
                        bRecordingStack = false;
                        CallStack = "";
                        RelevantData.Add("\n");
                        NamespacedHitch.Clear();
                        LastNumPrecedingWhitespaces = 0;
                    }
                    else if (StringToUse.Contains("--------------"))
                    {
                        RelevantData.Add(StringToUse);
                    }

                }
                else
                {
                    // This is a new thread stack. Let's mention where it came from.
                    if (TempString.Contains("-----Thread Hitch"))
                    {
                        bRecordingStack = true;
                        string[] stringSeparators = new string[] { "LogStats:" };
                        RelevantData.Add("Hitch from line " + LineNum.ToString() + "\n");
                        CallStack += TempString.Split(stringSeparators, StringSplitOptions.RemoveEmptyEntries)[TempString.Split(stringSeparators, StringSplitOptions.RemoveEmptyEntries).Length - 1] + "\n";
                    }
                }
            }
            NewReader.Close();
            string LogFileName = Path.GetFileName(FileLoc);
            LogFileName = LogFileName.Split('.')[0];
            StreamWriter NewWriter = new StreamWriter(LogFileName+ "_hitches.txt");
            for (int i = 0; i < RelevantData.Count; i++)
            {
                NewWriter.Write(RelevantData[i]);
            }
            NewWriter.Close();
            return RelevantData;
        }
    }
}
